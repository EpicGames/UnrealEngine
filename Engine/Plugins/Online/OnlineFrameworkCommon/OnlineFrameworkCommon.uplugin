{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Online Framework Common",
	"Description": "Common functionality for Online Frameworks",
	"Category": "Online",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "OnlineFrameworkCommon",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServices",
			"Enabled": true
		},
		{
			"Name": "OnlineServicesNull",
			"Enabled": true
		},
		{
			"Name": "OnlineServicesOSSAdapter",
			"Enabled": true
		}
	]
}