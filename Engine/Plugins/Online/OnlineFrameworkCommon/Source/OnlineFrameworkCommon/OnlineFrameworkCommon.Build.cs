// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class OnlineFrameworkCommon : ModuleRules
{
	public OnlineFrameworkCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreOnline",
				"CoreUObject",
			});

		if (Target.bCompileAgainstEngine)
		{
			PublicDependencyModuleNames.Add("Engine");
		}

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"OnlineServicesCommonEngineUtils",
				"OnlineServicesInterface",
				"OnlineServicesNull",
				"OnlineServicesOSSAdapter",
				"OnlineSubsystem",
			});
	}
}
