// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFramework/CommonAccount.h"

#include "Async/Future.h"
#include "OnlineFramework/CommonAccountManager.h"
#include "OnlineFramework/CommonAccountUtils.h"

namespace UE::OnlineFramework {

FCommonAccount::FCommonAccount(FPrivateToken, const FCommonAccountManagerRef& InManager, const FCommonConfig& InCommonConfig)
	: Manager(InManager)
	, CommonConfig(InCommonConfig)
{
}

UE::Online::FAccountId FCommonAccount::GetId(FName FrameworkInstance) const
{
	check(!FrameworkInstance.IsNone());
	FCommonAccountIdType AccountIdType = GetCommonAccountIdType(CommonConfig, FrameworkInstance);
	if (const UE::Online::FAccountId* const Id = AccountIds.Find(AccountIdType))
	{
		return *Id;
	}
	else if (RedirectAccount.IsValid())
	{
		if (FCommonAccountPtr PinnedRedirectAccount = RedirectAccount.Pin())
		{
			return PinnedRedirectAccount->GetId(FrameworkInstance);
		}
	}
	return UE::Online::FAccountId{};
}

void FCommonAccount::GetIdAsync(FName FrameworkInstance, FOnGetIdAsyncComplete&& OnComplete)
{
	UE::Online::FAccountId ExistingAccountId = GetId(FrameworkInstance);
	if (ExistingAccountId.IsValid())
	{
		OnComplete.ExecuteIfBound(AsShared(), ExistingAccountId);
		return;
	}
	else if (RedirectAccount.IsValid())
	{
		if (FCommonAccountPtr PinnedRedirectAccount = RedirectAccount.Pin())
		{
			PinnedRedirectAccount->GetIdAsync(FrameworkInstance, MoveTemp(OnComplete));
			return;
		}
	}
	FCommonAccountManagerPtr StrongManager = Manager.Pin();
	if (LIKELY(StrongManager.IsValid()))
	{
		StrongManager->LookupIdAsync(AsShared(), FrameworkInstance).Next([OnComplete = MoveTemp(OnComplete)](FCommonAccountManager::FLookupIdAsync&& Result)
		{
			OnComplete.ExecuteIfBound(Result.Key, Result.Value);
		});
		return;
	}
	OnComplete.ExecuteIfBound(AsShared(), UE::Online::FAccountId{});
}

bool FCommonAccount::AddId(FName FrameworkInstance, UE::Online::FAccountId AccountId)
{
	UE::Online::FAccountId ExistingAccountId = GetId(FrameworkInstance);
	if (ExistingAccountId.IsValid())
	{
		ensure(ExistingAccountId == AccountId);
		return false;
	}
	else if (RedirectAccount.IsValid())
	{
		if (FCommonAccountPtr PinnedRedirectAccount = RedirectAccount.Pin())
		{
			return PinnedRedirectAccount->AddId(FrameworkInstance, AccountId);
		}
	}

	FCommonAccountManagerPtr StrongManager = Manager.Pin();
	TOptional<FCommonConfigInstance> ConfigInstance = CommonConfig.GetFrameworkInstanceConfig(FrameworkInstance);
	if (!StrongManager || !ConfigInstance)
	{
		return false;
	}

	StrongManager->AddAccountId(AsShared(), AccountId, MoveTemp(ConfigInstance.GetValue()));
	return true;
}

bool FCommonAccount::Equals(const FCommonAccount& OtherAccount) const
{
	if (this == &OtherAccount)
	{
		return true;
	}
	if (TSharedPtr<const FCommonAccount> PinnedRedirectAccount = RedirectAccount.Pin())
	{
		return PinnedRedirectAccount->Equals(OtherAccount);
	}
	if (TSharedPtr<const FCommonAccount> PinnedOtherRedirectAccount = OtherAccount.RedirectAccount.Pin())
	{
		return PinnedOtherRedirectAccount->Equals(*this);
	}
	return false;
}

void FCommonAccount::AddAccountId(UE::Online::FAccountId AccountId, const FCommonAccountIdType& AccountIdType)
{
	check(!AccountIds.Contains(AccountIdType));
	AccountIds.Emplace(AccountIdType, AccountId);
}

FString FCommonAccount::ToLogString() const
{
	if (RedirectAccount.IsValid())
	{
		if (FCommonAccountPtr PinnedRedirectAccount = RedirectAccount.Pin())
		{
			return PinnedRedirectAccount->ToLogString();
		}
	}
	return FString::JoinBy(AccountIds, TEXT("&"), [](const TPair<FCommonAccountIdType, UE::Online::FAccountId>& Pair)
	{
		const UE::Online::EOnlineServices& OnlineServicesType = Pair.Key.Key;
		const FName& OnlineServicesConfigInstance = Pair.Key.Value;
		const UE::Online::FAccountId& AccountId = Pair.Value;
		return FString::Printf(TEXT("%s[%s]:%s"),
			LexToString(OnlineServicesType),
			*OnlineServicesConfigInstance.ToString(), // Will typically be None, but include it to have a consistently parsable format
			*UE::Online::ToLogString(AccountId));
	});
}

/* UE::OnlineFramework */ }