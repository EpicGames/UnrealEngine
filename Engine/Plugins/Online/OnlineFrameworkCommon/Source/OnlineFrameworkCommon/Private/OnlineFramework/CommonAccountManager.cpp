// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFramework/CommonAccountManager.h"

#include "OnlineFramework/CommonModule.h"
#include "OnlineFramework/CommonAccount.h"
#include "OnlineFramework/CommonAccountUtils.h"

namespace UE::OnlineFramework {

// FCommonAccountLookupAccountIdFnHandle
FCommonAccountLookupAccountIdFnHandle::FCommonAccountLookupAccountIdFnHandle(FCommonAccountManagerRef InManager, int InRegisteredId)
: Manager(InManager)
, RegisteredId(InRegisteredId)
{
}

void FCommonAccountLookupAccountIdFnHandle::Unbind()
{
	if (FCommonAccountManagerPtr PinnedManager = Manager.Pin())
	{
		PinnedManager->Unbind(RegisteredId);
	}
	Manager.Reset();
	RegisteredId = 0;
}

void FCommonAccountLookupAccountIdFnHandle::Reassign(FCommonAccountLookupAccountIdFnHandle&& InHandle)
{
	Unbind();
	Manager = InHandle.Manager;
	RegisteredId = InHandle.RegisteredId;
	InHandle.Manager.Reset();
	InHandle.RegisteredId = 0;
}

// FCommonAccountManager
int32 FCommonAccountManager::RegisteredIdCounter = 1;

FCommonAccountManager::FCommonAccountManager(FPrivateToken, const FCommonConfig& InCommonConfig)
	: CommonConfig(InCommonConfig)
{
}

FCommonAccountManager::~FCommonAccountManager()
{
	bCanPerformAsyncLookup = false;
	FailPendingLookups();
}

FCommonAccountPtr FCommonAccountManager::GetAccount(UE::Online::FAccountId AccountId, FName FrameworkInstance)
{
	if (!AccountId.IsValid())
	{
		// Invalid to have a account for an invalid id. For example, any actual account without a linked account for a system would all map to invalid id.
		return {};
	}
	if (FCommonAccountPtr ExistingAccount = FindExistingAccount(AccountId))
	{
		return ExistingAccount;
	}
	return CreateNewAccount(AccountId, FrameworkInstance);
}

FCommonAccountRef FCommonAccountManager::CreateNewAccount(UE::Online::FAccountId AccountId, FName FrameworkInstance)
{
	const FCommonAccountRef NewAccount = MakeShared<FCommonAccount>(FCommonAccount::FPrivateToken{}, AsShared(), CommonConfig);
	const FCommonAccountIdType AccountIdType = GetCommonAccountIdType(CommonConfig, FrameworkInstance);
	check(AccountIdType.Key == AccountId.GetOnlineServicesType());
	NewAccount->AddAccountId(AccountId, AccountIdType);
	Accounts.Emplace(AccountId, NewAccount);
	OnCommonAccountCreated().Broadcast(NewAccount);
	OnCommonAccountIdAdded().Broadcast(NewAccount, AccountId);
	return NewAccount;
}

FCommonAccountPtr FCommonAccountManager::FindExistingAccount(UE::Online::FAccountId AccountId) const
{
	if (const FCommonAccountRef* ExistingAccount = Accounts.Find(AccountId))
	{
		return *ExistingAccount;
	}
	return {};
}

TFuture<FCommonAccountManager::FLookupIdAsync> FCommonAccountManager::LookupIdAsync(const FCommonAccountRef& Account, FName FrameworkInstance)
{
	if (bCanPerformAsyncLookup)
	{
		TOptional<FCommonConfigInstance> ConfigInstance = CommonConfig.GetFrameworkInstanceConfig(FrameworkInstance);
		if (ConfigInstance)
		{
			TSharedRef<TPromise<FLookupIdAsync>> Promise = MakeShared<TPromise<FLookupIdAsync>>();
			PendingLookups.Emplace(Promise, Account);
			ExecuteNextLookup(Account, Promise, FrameworkInstance, MoveTemp(ConfigInstance.GetValue()), 0);
			return Promise->GetFuture();
		}
	}
	return MakeFulfilledPromise<FLookupIdAsync>(Account, UE::Online::FAccountId{}).GetFuture();
}

void FCommonAccountManager::ExecuteNextLookup(const FCommonAccountRef& Account, TSharedRef<TPromise<FLookupIdAsync>> Promise, FName FrameworkInstance, FCommonConfigInstance&& ConfigInstance, int InNextRegisteredId)
{
	// Loop so we can handle any lookup functions returning immediate results. This will avoid exceeding stack limits.
	int NextRegisteredId = InNextRegisteredId;
	while (true)
	{
		// Find the next function
		// This is generally expected to be a small list, and we do support functions being unregistered so we'll just start from the beginning each time.
		int32 NextFunctionIdx;
		for (NextFunctionIdx = 0; NextFunctionIdx < AccountIdLookups.Num(); ++NextFunctionIdx)
		{
			if (AccountIdLookups[NextFunctionIdx].RegisteredId >= NextRegisteredId)
			{
				break;
			}
		}

		if (NextFunctionIdx >= AccountIdLookups.Num())
		{
			FCommonAccountPtr OriginatingAccount;
			verify(PendingLookups.RemoveAndCopyValue(Promise, OriginatingAccount));
			Promise->EmplaceValue(OriginatingAccount.ToSharedRef(), UE::Online::FAccountId{});
			return;
		}

		NextRegisteredId = AccountIdLookups[NextFunctionIdx].RegisteredId + 1;
		TFuture<UE::Online::FAccountId> LookupFuture = AccountIdLookups[NextFunctionIdx].Function(*Account, FrameworkInstance, ConfigInstance);
		if (LookupFuture.IsReady())
		{
			// Handle immediately
			UE::Online::FAccountId FoundAccountId = LookupFuture.Get();
			if (FoundAccountId.IsValid())
			{
				if (FoundAccountId.GetOnlineServicesType() == ConfigInstance.OnlineServices)
				{
					HandleAccountIdFound(Account, FoundAccountId, MoveTemp(ConfigInstance), Promise);
					// Lookup complete, promise fulfilled
					return;
				}
				else
				{
					// The lookup method looked up the wrong online services type.
					checkNoEntry();
				}
			}
			// Proceed to the next lookup
		}
		else
		{
			// Handle asynchronously
			LookupFuture.Next([WeakThis = AsWeak(), Account, WeakPromise = Promise.ToWeakPtr(), FrameworkInstance, ConfigInstance = MoveTemp(ConfigInstance), NextRegisteredId](UE::Online::FAccountId FoundAccountId) mutable
			{
				TSharedPtr<TPromise<FLookupIdAsync>> StrongPromise = WeakPromise.Pin();
				if (!StrongPromise)
				{
					// Nothing is waiting for this result anymore
					return;
				}

				FCommonAccountManagerPtr StrongThis = WeakThis.Pin();
				check(StrongThis.IsValid()); // Expected to be valid if the promise is valid.
				if (FoundAccountId.IsValid())
				{
					if (FoundAccountId.GetOnlineServicesType() == ConfigInstance.OnlineServices)
					{
						StrongThis->HandleAccountIdFound(Account, FoundAccountId, MoveTemp(ConfigInstance), StrongPromise.ToSharedRef());
						// Lookup complete, promise fulfilled
						return;
					}
					else
					{
						// The lookup method looked up the wrong online services type.
						checkNoEntry();
					}
				}
				// Did not handle a success case, need to try the next function.
				StrongThis->ExecuteNextLookup(Account, StrongPromise.ToSharedRef(), FrameworkInstance, MoveTemp(ConfigInstance), NextRegisteredId);
			});
			return;
		}
	}
}

void FCommonAccountManager::HandleAccountIdFound(const FCommonAccountRef& InAccount, UE::Online::FAccountId AccountId, FCommonConfigInstance&& ConfigInstance, const TSharedRef<TPromise<FLookupIdAsync>>& LookupPromise)
{
	verify(PendingLookups.Remove(LookupPromise) > 0);
	FCommonAccountRef Account = AddAccountId(InAccount, AccountId, MoveTemp(ConfigInstance));
	LookupPromise->EmplaceValue(Account, AccountId);
}

FCommonAccountRef FCommonAccountManager::AddAccountId(const FCommonAccountRef& InAccount, UE::Online::FAccountId AccountId, FCommonConfigInstance&& ConfigInstance)
{
	FCommonAccountRef Account = InAccount;
	const bool bRequiresResolution = Accounts.Contains(AccountId);
	if (bRequiresResolution)
	{
		Account = ResolveAccountIdAssociationConflict(Account, AccountId);
	}
	else
	{
		check(ConfigInstance.OnlineServices == AccountId.GetOnlineServicesType());
		Account->AddAccountId(AccountId, GetCommonAccountIdType(ConfigInstance));
	}

	Accounts.Emplace(AccountId, Account);
	OnCommonAccountIdAdded().Broadcast(Account, AccountId);
	return Account;
}

namespace
{

FCommonAccountRef PickBestAccount(const FCommonAccountRef& Account1, const FCommonAccountRef& Account2)
{
	// TODO: Some algorithm that prefers one over the other?
	return Account2;
}

}

FCommonAccountRef FCommonAccountManager::ResolveAccountIdAssociationConflict(const FCommonAccountRef& Account, UE::Online::FAccountId ConflictingAccountId)
{
	FCommonAccountRef ConflictingAccount = Accounts.FindChecked(ConflictingAccountId);
	if (ConflictingAccount == Account)
	{
		return Account;
	}
	// Pick 'best' account to keep around, and 'merge' the other account into this one
	FCommonAccountRef BestAccount = PickBestAccount(Account, ConflictingAccount);
	const FCommonAccountRef& ReplacedAccount = (BestAccount == Account) ? ConflictingAccount : Account;
	check(!ReplacedAccount->RedirectAccount.IsValid());
	UE_LOGF(LogOnlineFrameworkCommon, Verbose, "ResolveAccountIdAssociationConflict: Account=[%ls] replaced with Account=[%ls]", *ReplacedAccount->ToLogString(), *BestAccount->ToLogString());
	ReplacedAccount->RedirectAccount = BestAccount;
	// Make BestAccount aware of all the ids of the other account, and broadcast an event that the other account has been replaced.
	TMap<FCommonAccountIdType, UE::Online::FAccountId> AddedAccountIds = ReplacedAccount->AccountIds; // Copy as events may modify this
	for (TPair<FCommonAccountIdType, UE::Online::FAccountId>& AccountId : AddedAccountIds)
	{
		BestAccount->AddAccountId(AccountId.Value, AccountId.Key);
		// Replace our association with this account id with the BestAccount
		Accounts.Emplace(AccountId.Value, BestAccount);
	}
	for (TPair<FCommonAccountIdType, UE::Online::FAccountId>& AccountId : AddedAccountIds)
	{
		OnCommonAccountIdAdded().Broadcast(BestAccount, AccountId.Value);
	}
	OnCommonAccountDuplicateDetected().Broadcast(BestAccount, ReplacedAccount);
	return BestAccount;
}

FCommonAccountLookupAccountIdFnHandle FCommonAccountManager::RegisterAccountIdLookup(FStringView Name, FCommonAccountLookupAccountIdFn&& InAccountIdFn)
{
	check(InAccountIdFn);
	int RegisteredId = RegisteredIdCounter++;
	UE_LOGF(LogOnlineFrameworkCommon, VeryVerbose, "Registered AccountIdLookup=[%.*ls] Id=%d", Name.Len(), Name.GetData(), RegisteredId);
	FRegisteredLookupFunction& RegisteredFunction = AccountIdLookups.Emplace_GetRef();
	RegisteredFunction.Name = Name; 
	RegisteredFunction.Function = MoveTemp(InAccountIdFn); 
	RegisteredFunction.RegisteredId = RegisteredId;
	return FCommonAccountLookupAccountIdFnHandle(AsShared(), RegisteredId);
}

void FCommonAccountManager::FailPendingLookups()
{
	TMap<TSharedRef<TPromise<FLookupIdAsync>>, FCommonAccountPtr> LocalPendingLookups = MoveTemp(PendingLookups);
	PendingLookups.Reset();
	for (TPair<TSharedRef<TPromise<FLookupIdAsync>>, FCommonAccountPtr>& PendingLookup : LocalPendingLookups)
	{
		PendingLookup.Key->EmplaceValue(PendingLookup.Value.ToSharedRef(), UE::Online::FAccountId{});
	}
}

void FCommonAccountManager::Unbind(int RegisteredId)
{
	int32 Index = AccountIdLookups.IndexOfByKey(RegisteredId);
	if (Index != INDEX_NONE)
	{
		UE_LOGF(LogOnlineFrameworkCommon, VeryVerbose, "Unregistered AccountIdLookup=[%ls] Id=%d", *AccountIdLookups[Index].Name, RegisteredId);
		AccountIdLookups.RemoveAt(Index);
	}
	else
	{
		// This can only be called by the RAII delegate handle
		checkNoEntry();
	}
}

FCommonAccountManagerPtr FCommonAccountManager::Get(const FCommonConfig& InCommonConfig)
{
	FOnlineFrameworkCommonModule* Module = FOnlineFrameworkCommonModule::Get();
	if (!Module)
	{
		return nullptr;
	}

	return Module->CachedAccountManagers.FindOrAdd(InCommonConfig.GetWorldContextName(),
		[&InCommonConfig]()
		{
			TSharedRef<FCommonAccountManager> AccountManager = MakeShared<FCommonAccountManager>(FPrivateToken{}, InCommonConfig);
			return AccountManager;
		});
}

void FCommonAccountManager::ClearRegistered(const FCommonConfig& InCommonConfig)
{
	if (FOnlineFrameworkCommonModule* const Module = FOnlineFrameworkCommonModule::Get())
	{
		Module->CachedAccountManagers.Clear(InCommonConfig.GetWorldContextName());
	}
}

/* UE::OnlineFramework */ }