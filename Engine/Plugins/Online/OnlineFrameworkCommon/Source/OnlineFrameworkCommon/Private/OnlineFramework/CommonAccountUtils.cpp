// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFramework/CommonAccountUtils.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "Online/AuthNull.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/OnlineServices.h"
#include "OnlineFramework/CommonAccount.h"
#include "OnlineFramework/CommonAccountManager.h"
#include "OnlineFramework/CommonConfig.h"
#include "OnlineFramework/CommonModule.h"
#include "OnlineSubsystem.h"

namespace UE::OnlineFramework
{

FCommonAccountIdType GetCommonAccountIdType(const FCommonConfig& CommonConfig, FName FrameworkInstance)
{
	if (TOptional<FCommonConfigInstance> CommonConfigInstance = CommonConfig.GetFrameworkInstanceConfig(FrameworkInstance))
	{
		return GetCommonAccountIdType(CommonConfigInstance.GetValue());
	}
	return {};
}

FCommonAccountIdType GetCommonAccountIdType(const FCommonConfigInstance& CommonConfigInstance)
{
	return { CommonConfigInstance.OnlineServices, CommonConfigInstance.OnlineServicesInstanceConfigName };
}

UE::Online::FAccountId GetAccountV2FromV1(const FCommonConfig& CommonConfig, const FUniqueNetIdPtr& InId, FName FrameworkInstance)
{
	UE::Online::FAccountId AccountId;

	if (InId.IsValid())
	{
		if (UE::Online::IOnlineServicesPtr OnlineServices = CommonConfig.GetServices(FrameworkInstance))
		{
			if (OnlineServices->GetServicesProvider() == UE::Online::EOnlineServices::Null && InId->GetType() == FName("Null"))
			{
				AccountId = UE::Online::FOnlineAccountIdRegistryNull::Get().FindOrAddAccountId(InId->ToString());
			}
			else if (UE::Online::FAuthOSSAdapterPtr AuthAdapter = OnlineServices->GetInterface<UE::Online::FAuthOSSAdapter>())
			{
				AccountId = AuthAdapter->GetAccountId(InId.ToSharedRef());
			}
		}
	}
	return AccountId;
}

FCommonAccountPtr GetCommonAccountFromV1(const FCommonConfig& CommonConfig, const FUniqueNetIdPtr& InId, FName FrameworkInstance)
{
	if (UE::Online::FAccountId AccountId = GetAccountV2FromV1(CommonConfig, InId, FrameworkInstance))
	{
		if (FCommonAccountManagerPtr CommonAccountManager = FCommonAccountManager::Get(CommonConfig))
		{
			return CommonAccountManager->GetAccount(AccountId, FrameworkInstance);
		}
	}
	return nullptr;
}

FCommonAccountPtr GetCommonAccountFromV2(const FCommonConfig& CommonConfig, const UE::Online::FAccountId& InAccountId, FName FrameworkInstance)
{
	if (InAccountId.IsValid())
	{
		if (TSharedPtr<FCommonAccountManager> CommonAccountManager = FCommonAccountManager::Get(CommonConfig))
		{
			return CommonAccountManager->GetAccount(InAccountId, FrameworkInstance);
		}
	}
	return nullptr;
}

FUniqueNetIdPtr GetV1FromCommonAccount(const FCommonAccountRef& CommonAccount, FName FrameworkInstance)
{
	FUniqueNetIdPtr AccountIdV1;
	if (UE::Online::FAccountId AccountIdV2 = CommonAccount->GetId(FrameworkInstance))
	{
		if (UE::Online::IOnlineServicesPtr OnlineServices = CommonAccount->GetCommonConfig().GetServices(FrameworkInstance))
		{
			// Special case Null: This does not use the adapter, and it's intended to be used for testing so no state consistency is expected between OnlineSubsystem and OnlineServices
			if (OnlineServices->GetServicesProvider() == UE::Online::EOnlineServices::Null)
			{
				if (IOnlineSubsystem* NullSubsystem = IOnlineSubsystem::Get(NULL_SUBSYSTEM))
				{
					if (IOnlineIdentityPtr Identity = NullSubsystem->GetIdentityInterface())
					{
						const FString& NullAccountIdString = UE::Online::FOnlineAccountIdRegistryNull::Get().ToString(AccountIdV2);
						AccountIdV1 = Identity->CreateUniquePlayerId(NullAccountIdString);
					}
				}
			}
			else if (UE::Online::FAuthOSSAdapterPtr AuthAdapter = OnlineServices->GetInterface<UE::Online::FAuthOSSAdapter>())
			{
				AccountIdV1 = AuthAdapter->GetUniqueNetId(AccountIdV2);
			}
		}
	}

	return AccountIdV1;
}

FName GetFirstFrameworkInstanceName(const FUniqueNetIdWrapper& InId)
{
	// Framework instance names require a OnlineServices name, so the input must either:
	// Be a V2 id
	// Be a OnlineSubsystemNull V1 id
	// Be a V1 id where there is an OnlineServices adapter implementation available for the online subsystem type
	FName FrameworkInstance;
	if (InId.IsValid())
	{
		if (InId.IsV1())
		{
			// Requires a common config entry with this name
			FrameworkInstance = InId.GetV1Unsafe()->GetType();
		}
		else if (FOnlineFrameworkCommonModule* CommonModule = FOnlineFrameworkCommonModule::Get())
		{
			UE::Online::EOnlineServices Services = InId.GetV2Unsafe().GetOnlineServicesType();
			for (const TPair<TPair<FName, ECommonConfigContextType>, FCommonConfigInstance>& Config : CommonModule->GetConfig().FrameworkConfigs)
			{
				const FCommonConfigInstance& ConfigInstance = Config.Value;
				if (ConfigInstance.OnlineServices == Services)
				{
					FrameworkInstance = Config.Key.Key;
					break;
				}
			}
		}
	}
	return FrameworkInstance;
}

}
