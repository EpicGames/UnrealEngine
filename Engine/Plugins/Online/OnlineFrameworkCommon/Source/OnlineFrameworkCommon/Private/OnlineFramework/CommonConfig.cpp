// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFramework/CommonConfig.h"

#if WITH_ENGINE
#include "Engine/Engine.h"
#include "Engine/World.h"
#endif

#include "Online/OnlineServices.h"
#include "OnlineFramework/CommonModule.h"
#include "Templates/SharedPointer.h"

namespace UE::OnlineFramework {

const TCHAR* LexToString(const ECommonConfigContextType Value)
{
	switch (Value)
	{
	default: [[fallthrough]];
	case ECommonConfigContextType::Default: return TEXT("Default");
	case ECommonConfigContextType::Client:  return TEXT("Client");
	case ECommonConfigContextType::Server:  return TEXT("Server");
	case ECommonConfigContextType::Editor:  return TEXT("Editor");
	}
}

bool LexTryParseString(ECommonConfigContextType& OutValue, FStringView StringView)
{
	if (StringView == TEXT("Default"))
	{
		OutValue = ECommonConfigContextType::Default;
	}
	else if (StringView == TEXT("Client"))
	{
		OutValue = ECommonConfigContextType::Client;
	}
	else if (StringView == TEXT("Server"))
	{
		OutValue = ECommonConfigContextType::Server;
	}
	else if (StringView == TEXT("Editor"))
	{
		OutValue = ECommonConfigContextType::Editor;
	}
	else
	{
		return false;
	}
	return true;
}

void LexFromString(ECommonConfigContextType& OutValue, FStringView StringView)
{
	if (!ensureAlwaysMsgf(LexTryParseString(OutValue, StringView), TEXT("Unable to parse ECommonConfigContextType value: %.*s"), StringView.Len(), StringView.GetData()))
	{
		OutValue = ECommonConfigContextType::Default;
	}
}

FCommonConfig::FCommonConfig(const UObject* ContextObject)
{
	// Use the default of NAME_None for WorldContextName with all contexts except PIE worlds, which is set down below.

	if (IsRunningDedicatedServer())
	{
		ContextType = ECommonConfigContextType::Server;
		return;
	}

	if (IsRunningGame())
	{
		ContextType = ECommonConfigContextType::Client;
		return;
	}

#if WITH_EDITOR
	if (ContextObject)
	{
		if (const UWorld* const World = ContextObject->GetWorld())
		{
			if (World->IsPlayInEditor())
			{
				const FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);
				if (WorldContext)
				{
					WorldContextName = WorldContext->ContextHandle;
					ContextType = WorldContext->RunAsDedicated ? ECommonConfigContextType::Server : ECommonConfigContextType::Client;
				}
				else
				{
					// This should not happen unless there are edge cases during shutdown. We should consider using an invalid state here.
					UE_LOGF(LogOnlineFrameworkCommon, Warning, "[FCommonFrameworkContext] PIE world without a world context");
					ContextType = ECommonConfigContextType::Editor;
				}
			}
			else
			{
				// Context objects with worlds that are not PIE should use the global editor config.
				ContextType = ECommonConfigContextType::Editor;
			}
		}
		else
		{
			// Assume context objects without worlds should use the global editor config. This may not be correct, as there could be edge cases where
			// a context object that was part of a PIE world no longer has a world during shutdown. If this becomes an issue, we may need to introduce
			// an invalid context type that prevents it from being used.
			UE_LOGF(LogOnlineFrameworkCommon, Warning, "[FCommonFrameworkContext] Editor context without a world");
			ContextType = ECommonConfigContextType::Editor;
		}
	}
	else
	{
		// Not passing in a context object should use the global editor config available outside of PIE.
		ContextType = ECommonConfigContextType::Editor;
	}
#else
	// Not a client, server, or editor.
	ContextType = ECommonConfigContextType::Default;
#endif
}

TOptional<FCommonConfigInstance> FCommonConfig::GetFrameworkInstanceConfig(FName FrameworkInstance) const
{
	if (const FCommonConfigInstance* Found = FindFrameworkInstanceConfig(FrameworkInstance))
	{
		return { *Found };
	}
	return {};
}

const FCommonConfigInstance* FCommonConfig::FindFrameworkInstanceConfig(FName FrameworkInstance) const
{
	if (FOnlineFrameworkCommonModule* CommonModule = FOnlineFrameworkCommonModule::Get())
	{
		const FOnlineFrameworkCommonConfig& ModuleConfig = CommonModule->GetConfig();

		// First look by Name + Type
		if (ContextType != ECommonConfigContextType::Default)
		{
			if (const FCommonConfigInstance* const FoundConfigInstance = ModuleConfig.FrameworkConfigs.Find({ FrameworkInstance, ContextType }))
			{
				return FoundConfigInstance;
			}
		}
		// fallback on Name + Default
		if (const FCommonConfigInstance* const FoundConfigInstance = ModuleConfig.FrameworkConfigs.Find({ FrameworkInstance, ECommonConfigContextType::Default }))
		{
			return FoundConfigInstance;
		}
	}
	return nullptr;
}

TSharedPtr<UE::Online::IOnlineServices> FCommonConfig::GetServices(FName FrameworkInstance) const
{
	using namespace UE::Online;
	TSharedPtr<IOnlineServices> OnlineServices;
	if (const FCommonConfigInstance* FrameworkInstanceConfig = FindFrameworkInstanceConfig(FrameworkInstance))
	{
		OnlineServices = UE::Online::GetServices(FrameworkInstanceConfig->OnlineServices, WorldContextName, FrameworkInstanceConfig->OnlineServicesInstanceConfigName);
	}
	return OnlineServices;
}

/* UE::OnlineFramework */ }