// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineFramework/CommonModule.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "OnlineFramework/CommonAccountManager.h"
#include "OnlineFramework/CommonConfig.h"

DEFINE_LOG_CATEGORY(LogOnlineFrameworkCommon);

namespace UE::OnlineFramework
{

static const TCHAR* OnlineFrameworkCommonConfigIniSectionName = TEXT("OnlineFrameworkCommonConfig");

FOnlineFrameworkCommonModule::~FOnlineFrameworkCommonModule() = default;

FOnlineFrameworkCommonModule* FOnlineFrameworkCommonModule::Get()
{
	FOnlineFrameworkCommonModule* Module = FModuleManager::GetModulePtr<FOnlineFrameworkCommonModule>("OnlineFrameworkCommon");
	UE_CLOGF(!Module, LogOnlineFrameworkCommon, Error, "%s Module not loaded.", __FUNCTION__);
	return Module;
}

void FOnlineFrameworkCommonModule::StartupModule()
{
	FCoreDelegates::TSOnConfigSectionsChanged().AddRaw(this, &FOnlineFrameworkCommonModule::OnConfigSectionsChanged);
	UpdateFromConfig();
}

void FOnlineFrameworkCommonModule::ShutdownModule()
{
	FCoreDelegates::TSOnConfigSectionsChanged().RemoveAll(this);
}

void FOnlineFrameworkCommonModule::OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames)
{
	if (IniFilename == GGameIni && SectionNames.Contains(OnlineFrameworkCommonConfigIniSectionName))
	{
		UpdateFromConfig();
	}
}

void FOnlineFrameworkCommonModule::UpdateFromConfig()
{
	using namespace UE::Online;
	Config = {};

	TArray<FString> ConfigStrings;
	GConfig->GetArray(OnlineFrameworkCommonConfigIniSectionName, TEXT("Instances"), ConfigStrings, GGameIni);
	for (FString& ConfigString : ConfigStrings)
	{
		// Expected Format:  +Instances=(Name=a, OnlineServices=b, ConfigInstance=c, Type=d)
		// Trim ( )
		ConfigString.TrimStartAndEndInline();
		if (ConfigString.Len() >= 2 && ConfigString[0] == '(' && ConfigString[ConfigString.Len() - 1] == ')')
		{
			ConfigString.MidInline(1, ConfigString.Len() - 2);
		}
		// Required: Name=, OnlineServices=
		// Optional: ConfigInstance=, Type=
		FName InstanceName;
		FString OnlineServicesString;
		if (FParse::Value(*ConfigString, TEXT("Name="), InstanceName) && ensureAlways(!InstanceName.IsNone()) &&
			FParse::Value(*ConfigString, TEXT("OnlineServices="), OnlineServicesString))
		{
			FCommonConfigInstance FrameworkConfig;
			LexFromString(FrameworkConfig.OnlineServices, *OnlineServicesString);
			if (FrameworkConfig.OnlineServices == EOnlineServices::None)
			{
				ensureMsgf(OnlineServicesString == LexToString(EOnlineServices::None), TEXT("%hs Invalid type [%s]"), __FUNCTION__, *OnlineServicesString);
				continue;
			}
			FParse::Value(*ConfigString, TEXT("ConfigInstance="), FrameworkConfig.OnlineServicesInstanceConfigName);
			FString TypeStr;
			if (FParse::Value(*ConfigString, TEXT("Type="), TypeStr))
			{
				LexFromString(FrameworkConfig.Type, *TypeStr);
			}
			Config.FrameworkConfigs.Emplace({ InstanceName, FrameworkConfig.Type }, MoveTemp(FrameworkConfig));
		}
		else
		{
			UE_LOGF(LogOnlineFrameworkCommon, Warning, "%s Failed to parse line [%ls]", __FUNCTION__, *ConfigString);
		}
	}
}

}

IMPLEMENT_MODULE(UE::OnlineFramework::FOnlineFrameworkCommonModule, OnlineFrameworkCommon);
