// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "Online/WorldContextScopedObjectCache.h"
#include "OnlineFramework/CommonConfig.h"

DECLARE_LOG_CATEGORY_EXTERN(LogOnlineFrameworkCommon, Log, All);

namespace UE::OnlineFramework
{

enum class ECommonConfigContextType : uint8;
class FCommonAccountManager;
using FCommonAccountManagerPtr = TSharedPtr<FCommonAccountManager>;
using FCommonAccountManagerRef = TSharedRef<FCommonAccountManager>;
	
/** Config for OnlineFrameworkCommon */
struct FOnlineFrameworkCommonConfig
{
	/** Framework configs. Map of FrameworkInstance name+type to config instance. */
	TMap<TPair<FName, ECommonConfigContextType>, FCommonConfigInstance> FrameworkConfigs;
};

class FOnlineFrameworkCommonModule : public IModuleInterface
{
public:
	~FOnlineFrameworkCommonModule();

	static FOnlineFrameworkCommonModule* Get();
	const FOnlineFrameworkCommonConfig& GetConfig() const { return Config; }

	FWorldContextScopedObjectCache<FCommonAccountManager> CachedAccountManagers;
private:
	// ~Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// ~End IModuleInterface

	void OnConfigSectionsChanged(const FString& IniFilename, const TSet<FString>& SectionNames);
	void UpdateFromConfig();

	/** CommonConfig config */
	FOnlineFrameworkCommonConfig Config;
};

}
