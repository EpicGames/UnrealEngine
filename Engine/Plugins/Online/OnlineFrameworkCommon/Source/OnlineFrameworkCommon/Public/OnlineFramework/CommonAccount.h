// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "OnlineFramework/CommonConfig.h"

namespace UE::OnlineFramework {

class FCommonAccount;
using FCommonAccountPtr = TSharedPtr<FCommonAccount>;
using FCommonAccountRef = TSharedRef<FCommonAccount>;
class FCommonAccountManager;
using FCommonAccountManagerRef = TSharedRef<FCommonAccountManager>;

/**
 * Class that identifies a unique account and has the collection of account ids linked to that account
 */
class FCommonAccount
	: public TSharedFromThis<FCommonAccount>
{
public:
    /**
	 * Get the cached id for a framework instance
	 * @see GetIdAsync
	 * @see FCommonConfig
	 * @param FrameworkInstance the type of account id to retrieve
	 * @return the cached account id, or a default constructed id if there is no cached id
	 */
    [[nodiscard]] ONLINEFRAMEWORKCOMMON_API UE::Online::FAccountId GetId(FName FrameworkInstance) const;

	/**
	 * Completion delegate for GetIdAsync
	 * @param Account the account instance that currently represents this account. This is not guaranteed to be the account that GetIdAsync was called on.
	 *   For example, if we found a corresponding account id, that id may belong to another registered account. In that case, we must perform a reconciliation
	 *   and we may prefer the other account instance over this one.
	 * @param AccountId the looked up account id. This may be an invalid id if there is no association.
	 */
	DECLARE_DELEGATE_TwoParams(FOnGetIdAsyncComplete, const FCommonAccountRef& /*Account*/, UE::Online::FAccountId /*AccountId*/);

	/**
	 * Lookup an account id for this account. This will prefer any cached values, then call all registered lookup methods for the registered ids.
	 * @see GetId
	 * @see FCommonConfig
	 * @param FrameworkInstance the type of account id to retrieve
	 * @param OnComplete completion delegate to call on complete
	 */
	ONLINEFRAMEWORKCOMMON_API void GetIdAsync(FName FrameworkInstance, FOnGetIdAsyncComplete&& OnComplete);

	/**
	 * Add an account id for this account. If the id already exists for the framework instance, or if the manager or framework instance not valid, no changes are made and this returns false.
	 * @param FrameworkInstance the type of account id to add
	 * @param AccountId the account ID to add
	 * @return whether the account id was added
	*/
	ONLINEFRAMEWORKCOMMON_API bool AddId(FName FrameworkInstance, UE::Online::FAccountId AccountId);

	/**
	 * Check if the target account equals this one. Generally, there should only be one common account associated with an account id, but 
	 * after GetIdAsync we may merge them and systems may have lingering references to the replaced common account
	 * @param OtherAccount the account to test
	 * @return true if they are the same, false if not
	 */
	ONLINEFRAMEWORKCOMMON_API bool Equals(const FCommonAccount& OtherAccount) const;
	friend bool operator==(const FCommonAccount& Lhs, const FCommonAccount& Rhs)
	{
		return Lhs.Equals(Rhs);
	}

	/**
	 * Get a loggable version of this account
	 * @return a loggable version of this account
	 */
	ONLINEFRAMEWORKCOMMON_API FString ToLogString() const;

	friend FString ToLogString(const FCommonAccount& InCommonAccount)
	{
		return InCommonAccount.ToLogString();
	}
	friend uint32 GetTypeHash(const FCommonAccount& InCommonAccount)
	{
		return ::GetTypeHash(&InCommonAccount);
	}

	const FCommonConfig& GetCommonConfig()
	{
		return CommonConfig;
	}

private:
	using FCommonAccountIdType = TTuple<UE::Online::EOnlineServices /*OnlineServices*/, FName /*OnlineServicesInstanceConfigName*/>;
	void AddAccountId(UE::Online::FAccountId AccountId, const FCommonAccountIdType& AccountIdType);
	
	// In the case of a conflict, this account could be replaced by another account. In that case, GetId and GetIdAsync for this instance will be redirected to the
	// kept account. This is for the convenience of users for Common Account. The expected usage is to use the result of GetIdAsync 
	// or bind to FOnCommonAccountDuplicateDetected to handle account replacements.
	TWeakPtr<FCommonAccount> RedirectAccount;
	TWeakPtr<FCommonAccountManager> Manager;
	FCommonConfig CommonConfig;
	TMap<FCommonAccountIdType, UE::Online::FAccountId> AccountIds;

private:
	friend class FCommonAccountManager;
	struct FPrivateToken { explicit FPrivateToken() = default; };
public: // Public constructor so MakeShared can instantiate this, however this may only be instantiated by FCommonAccountManager
	explicit FCommonAccount(FPrivateToken, const FCommonAccountManagerRef& Inmanager, const FCommonConfig& InCommonConfig);
};

/* UE::OnlineFramework */ }