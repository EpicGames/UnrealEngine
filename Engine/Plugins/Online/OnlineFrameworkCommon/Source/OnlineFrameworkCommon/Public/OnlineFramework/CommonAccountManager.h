// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Online/CoreOnline.h"
#include "OnlineFramework/CommonConfig.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::OnlineFramework {

class FCommonAccount;
using FCommonAccountPtr = TSharedPtr<FCommonAccount>;
using FCommonAccountRef = TSharedRef<FCommonAccount>;
class FCommonAccountManager;
using FCommonAccountManagerPtr = TSharedPtr<FCommonAccountManager>;
using FCommonAccountManagerRef = TSharedRef<FCommonAccountManager>;

using FCommonAccountLookupAccountIdFn = TFunction<TFuture<UE::Online::FAccountId>(FCommonAccount& /*Account*/, FName /*RequestingFrameworkInstance*/, const FCommonConfigInstance& /*RequestingFrameworkInstanceData*/)>;

/** Handle for registered lookup function. Automatically unregisters on destruction, or explicitly by calling Unbind. */
class FCommonAccountLookupAccountIdFnHandle final : public FNoncopyable
{
public:
	FCommonAccountLookupAccountIdFnHandle() {};
	FCommonAccountLookupAccountIdFnHandle(FCommonAccountLookupAccountIdFnHandle&& InHandle) { Reassign(MoveTemp(InHandle)); }
	FCommonAccountLookupAccountIdFnHandle& operator=(FCommonAccountLookupAccountIdFnHandle&& InHandle) { Reassign(MoveTemp(InHandle)); return *this; }
	~FCommonAccountLookupAccountIdFnHandle() { Unbind(); }

	ONLINEFRAMEWORKCOMMON_API void Unbind();
private:
	ONLINEFRAMEWORKCOMMON_API void Reassign(FCommonAccountLookupAccountIdFnHandle&& InHandle);

	FCommonAccountLookupAccountIdFnHandle(FCommonAccountManagerRef InManager, int InRegisteredId);
	friend class FCommonAccountManager;
	TWeakPtr<FCommonAccountManager> Manager;
	int RegisteredId = 0;
};

/**
 * Event triggered when a common account is created
 * @param Account the created account
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnCommonAccountCreated, const FCommonAccountRef& /*Account*/);
/**
 * Event triggered when a common account has an account id added
 * @param Account the account that has an account id added
 * @param NewAccountId the new account id
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCommonAccountIdAdded, const FCommonAccountRef& /*Account*/, UE::Online::FAccountId /*NewAccountId*/);
/**
 * Event triggered when two common accounts have been identified to represent the same account
 * For example, if account1 is created for account A, and account2 for account B, and something later calls GetIdAsync for account1 for the type of account B, we may find that the accounts are linked and therefore are the same account
 * In this case, one account will be kept and the other account will be removed. Integrations are expected to update any local mappings that reference the removed account to reference the kept account. 
 * @param KeptAccount the account that was kept
 * @param RemovedAccount the account that was removed
 */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnCommonAccountDuplicateDetected, const FCommonAccountRef& /*KeptAccount*/, const FCommonAccountRef& /*RemovedAccount*/);

/**
 * Class to manage creation of and links between FCommonAccount
 */
class FCommonAccountManager
	: public TSharedFromThis<FCommonAccountManager>
{
	UE_NONCOPYABLE(FCommonAccountManager);
public:
	ONLINEFRAMEWORKCOMMON_API ~FCommonAccountManager();

	/**
	 * Get the account manager for a context
	 * @param InContext the context to get the account manager for.
	 * @return the common account manager for the context
	 */
	ONLINEFRAMEWORKCOMMON_API static FCommonAccountManagerPtr Get(const FCommonConfig& InCommonConfig);

	/**
	 * Clear the account manager for a context. Expected to only be used in strictly controlled scenarios, such as automated tests.
	 * @param InContext the context to clear the account manager for.
	 */
	ONLINEFRAMEWORKCOMMON_API static void ClearRegistered(const FCommonConfig& InCommonConfig);

    /**
	 * Get an existing or create a new account given an account id and FrameworkInstance
	 * @param AccountId the account id for the account
	 * @param FrameworkInstance the framework instance that defines the account id's type
	 * @return the common account for the id. Will return nullptr only if AccountId is invalid.
	 */
	[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FCommonAccountPtr GetAccount(UE::Online::FAccountId AccountId, FName FrameworkInstance);

	/**
	 * Register a function to perform lookups
	 * @param Name the name of the function for debugging
	 */
	[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FCommonAccountLookupAccountIdFnHandle RegisterAccountIdLookup(FStringView Name, FCommonAccountLookupAccountIdFn&& InAccountIdFn);

	FOnCommonAccountCreated& OnCommonAccountCreated() { return OnCommonAccountCreatedEvent; }
	FOnCommonAccountIdAdded& OnCommonAccountIdAdded() { return OnCommonAccountIdAddedEvent; }
	FOnCommonAccountDuplicateDetected& OnCommonAccountDuplicateDetected() { return OnCommonAccountDuplicateDetectedEvent; }
private:
	friend class FCommonAccount;
	friend class FCommonAccountLookupAccountIdFnHandle;
	// Create a new account
	FCommonAccountRef CreateNewAccount(UE::Online::FAccountId AccountId, FName FrameworkInstance);
	// Find an existing account
	FCommonAccountPtr FindExistingAccount(UE::Online::FAccountId AccountId) const;
	// Lookup an id for a account
	using FLookupIdAsync = TTuple<FCommonAccountRef, UE::Online::FAccountId>;
	TFuture<FLookupIdAsync> LookupIdAsync(const FCommonAccountRef& InAccount, FName FrameworkInstance);
	// Continue a lookup on the next registered function
	void ExecuteNextLookup(const FCommonAccountRef& InAccount, TSharedRef<TPromise<FLookupIdAsync>> Promise, FName FrameworkInstance, FCommonConfigInstance&& ConfigInstance, int LastRegisteredId);
	// Handle an async lookup completing
	void HandleAccountIdFound(const FCommonAccountRef& Account, UE::Online::FAccountId AccountId, FCommonConfigInstance&& ConfigInstance, const TSharedRef<TPromise<FLookupIdAsync>>& LookupPromise);
	// Add a new account ID
	FCommonAccountRef AddAccountId(const FCommonAccountRef& Account, UE::Online::FAccountId AccountId, FCommonConfigInstance&& ConfigInstance);
	// Resolve an account id association conflict where we find out two common accounts are actually the same account after discovering a linked account
	FCommonAccountRef ResolveAccountIdAssociationConflict(const FCommonAccountRef& Account, UE::Online::FAccountId ConflictingAccountId);

	// Unbind a registered function
	void Unbind(int RegisteredId);
	// Complete all pending lookups. Called on destruction to ensure we have no unfulfilled promises.
	void FailPendingLookups();

	// Common config to resolve FrameworkInstance to online services
	FCommonConfig CommonConfig;
	// Registered accounts. Note that while FAccountId lacks context to know which framework instance it is for, FAccountId is unique enough to index on.
	TMap<UE::Online::FAccountId, FCommonAccountRef> Accounts;
	// Registered account id lookups
	struct FRegisteredLookupFunction
	{
		FString Name;
		FCommonAccountLookupAccountIdFn Function;
		int RegisteredId;
		friend bool operator==(const FRegisteredLookupFunction& This, int InRegisteredId) { return This.RegisteredId == InRegisteredId; }
	};
	TArray<FRegisteredLookupFunction> AccountIdLookups;
	// In progress lookups
	TMap<TSharedRef<TPromise<FLookupIdAsync>>, FCommonAccountPtr> PendingLookups;
	static int32 RegisteredIdCounter;
	// Whether we can accept new async lookup requests or not
	bool bCanPerformAsyncLookup = true;

	FOnCommonAccountCreated OnCommonAccountCreatedEvent;
	FOnCommonAccountIdAdded OnCommonAccountIdAddedEvent;
	FOnCommonAccountDuplicateDetected OnCommonAccountDuplicateDetectedEvent;

private:
	struct FPrivateToken { explicit FPrivateToken() = default; };
public:
	// Constructor: Do not construct directly. Use Create instead.
	explicit FCommonAccountManager(FPrivateToken, const FCommonConfig& InCommonConfig);
};


/* UE::OnlineFramework */ }