// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"

namespace UE::OnlineFramework
{

class FCommonAccount;
using FCommonAccountRef = TSharedRef<FCommonAccount>;
using FCommonAccountPtr = TSharedPtr<FCommonAccount>;
class FCommonConfig;
struct FCommonConfigInstance;
struct FCommonFrameworkContext;

/** Tuple that uniquely identifies an account id type */
using FCommonAccountIdType = TTuple<UE::Online::EOnlineServices /*OnlineServices*/, FName /*OnlineServicesInstanceConfigName*/>;
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FCommonAccountIdType GetCommonAccountIdType(const FCommonConfig& CommonConfig, FName FrameworkInstance);
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FCommonAccountIdType GetCommonAccountIdType(const FCommonConfigInstance& CommonConfigInstance);
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API UE::Online::FAccountId GetAccountV2FromV1(const FCommonConfig& CommonConfig, const FUniqueNetIdPtr& InId, FName FrameworkInstance);
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FCommonAccountPtr GetCommonAccountFromV1(const FCommonConfig& CommonConfig, const FUniqueNetIdPtr& InId, FName FrameworkInstance);
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FCommonAccountPtr GetCommonAccountFromV2(const FCommonConfig& CommonConfig, const UE::Online::FAccountId& InAccountId, FName FrameworkInstance);
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FUniqueNetIdPtr GetV1FromCommonAccount(const FCommonAccountRef& CommonAccount, FName FrameworkInstance);
/**
 * Get the first framework instance that matches the input account id.
 * Unsafe if the account id's type could have multiple, incompatible online services instances associated with it. Otherwise, generally safe.
 * For V1 net ids, requires a common config entry with Name=OnlineSubsystemName
 * Prefer having your system know by config or other inputs what the framework instance is.
 * Intended to be a utility while converting existing systems over to using newer online frameworks.
 * @param InId the account id to get the framework instance name for
 * @return the first framework instance that matches the input's type, or NAME_None if none is found.
 */
[[nodiscard]] ONLINEFRAMEWORKCOMMON_API FName GetFirstFrameworkInstanceName(const FUniqueNetIdWrapper& InId);
}
