// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CoreOnline.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/NameTypes.h"

class UObject;

namespace UE::Online { class IOnlineServices; }

namespace UE::OnlineFramework {

/** Context type */
enum class ECommonConfigContextType : uint8
{
	/** No specific meaning or context */
	Default,
	/** Client (including listen server) */
	Client,
	/** Dedicated Server */
	Server,
	/** Editor */
	Editor,
};

ONLINEFRAMEWORKCOMMON_API const TCHAR* LexToString(const ECommonConfigContextType Value);
ONLINEFRAMEWORKCOMMON_API bool LexTryParseString(ECommonConfigContextType& OutValue, FStringView StringView);
ONLINEFRAMEWORKCOMMON_API void LexFromString(ECommonConfigContextType& OutValue, FStringView StringView);

struct FCommonConfigInstance
{
	/** The OnlineServices type */
	UE::Online::EOnlineServices OnlineServices = UE::Online::EOnlineServices::None;
	/** The config to use for this OnlineServices instance (could be NAME_None depending on the implementation) */
	FName OnlineServicesInstanceConfigName;
	/** The context this config instance applies to */
	ECommonConfigContextType Type = ECommonConfigContextType::Default;
};

class FCommonConfig
{
public:
	/** Constructor to explicitly set fields */
	FCommonConfig(FName InWorldContextName, ECommonConfigContextType InContextType)
		: WorldContextName(InWorldContextName)
		, ContextType(InContextType)
	{
	}
	/** Constructor to determine the context from a UObject. Intentionally implicit. */
	ONLINEFRAMEWORKCOMMON_API FCommonConfig(const UObject* ContextObject);

	FCommonConfig() = delete;

	/**
	 * Get the config for a framework instance
	 * @param FrameworkInstance the framework instance name
	 * @return the config, or an unset optional if the config is not present.
	 */
	ONLINEFRAMEWORKCOMMON_API TOptional<FCommonConfigInstance> GetFrameworkInstanceConfig(FName FrameworkInstance) const;

	/**
	 * Get the online services configured for a framework instance
	 * @param FrameworkInstance the framework instance name
	 * @return the online services. May be null if the framework instance is not configured, or if the online services is unavailable
	 */
	ONLINEFRAMEWORKCOMMON_API TSharedPtr<UE::Online::IOnlineServices> GetServices(FName FrameworkInstance) const;

	/**
	 * Get the world context for this online config instance.
	 * @return the world context
	 */
	FName GetWorldContextName() const
	{
		return WorldContextName;
	}

	/**
	 * Get the context type for this online config instance.
	 * @return the context type
	 */
	ECommonConfigContextType GetContextType() const
	{
		return ContextType;
	}

private:
	const FCommonConfigInstance* FindFrameworkInstanceConfig(FName FrameworkInstance) const;

	/** World context name. Expected to be NAME_None, except for PIE where it will be the world's context name. */
	FName WorldContextName;

	/** Context type */
	ECommonConfigContextType ContextType;
};

/* UE::OnlineFramework */ }